// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

using System.IO;

public class SlateTests : TestModuleRules
{
	static SlateTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "Slate";
			TestMetadata.TestShortName = "Slate";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}
		}
	}

	public SlateTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
			});

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform"
				});
		}

		PrivateIncludePaths.AddRange(
			new string[] {
				Path.Combine(Target.RelativeEnginePath, "Source", "Runtime", "Slate", "Private"),
			}
		);
	}
}
