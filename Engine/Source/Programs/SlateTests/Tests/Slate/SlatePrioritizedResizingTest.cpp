// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"

#include "Framework/MultiBox/SClippingHorizontalBox.h"

#include <iostream>

namespace UE::Slate
{

bool operator==(const UE::Slate::FClippingInfo& Left, const UE::Slate::FClippingInfo& Right)
{
	return Left.Widget == Right.Widget && Left.X == Right.X && Left.Width == Right.Width
		&& Left.bIsStretchable == Right.bIsStretchable && Left.bAppearsInOverflow == Right.bAppearsInOverflow
		&& Left.bWasClipped == Right.bWasClipped;
}

std::ostream& operator<<(std::ostream& Stream, const UE::Slate::FClippingInfo& Value)
{
	// Format booleans as "true" and "false" instead of 1 and 0.
	Stream << std::boolalpha;

	Stream << "FClippingInfo3("
		   << "Widget=" << Value.Widget.Get() << ", ";
	Stream << "FMenuEntryResizeParams(" << Value.ResizeParams.ClippingPriority.Get() << ", "
		   << Value.ResizeParams.AllowClipping.Get() << ", " << Value.ResizeParams.VisibleInOverflow.Get() << "), ";
	Stream << "X=" << Value.X << ", ";
	Stream << "Width=" << Value.Width << ", ";
	Stream << "bIsStretchable=" << Value.bIsStretchable << ", ";
	Stream << "bAppearsInOverflow=" << Value.bAppearsInOverflow << ", ";
	Stream << "bWasClipped=" << Value.bWasClipped;
	Stream << ")";
	return Stream;
}

} // namespace UE::Slate

TEST_CASE("Runtime::Slate::PrioritizedResizing works on empty input", "[Slate]")
{
	constexpr float AllottedWidth = 200.0f;
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(4.0f, 0);
	constexpr int32 WrapButtonIndex = -1;
	TArray<UE::Slate::FClippingInfo> ClippingInfos;

	TOptional<float> WrapButtonX;
	UE::Slate::PrioritizedResize(
		AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, ClippingInfos, WrapButtonX
	);

	CHECK(ClippingInfos.IsEmpty());
}

TEST_CASE("Runtime::Slate::PrioritizedResizing clips no widgets when there's plenty of space", "[Slate]")
{
	constexpr float AllottedWidth = 500.0f;
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(4.0f, 0);
	constexpr int32 WrapButtonIndex = -1;
	TArray<UE::Slate::FClippingInfo> ClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f,  .Width = 20.0f  },
		UE::Slate::FClippingInfo{ .X = 30.0f, .Width = 100.0f },
	};
	TOptional<float> WrapButtonX;

	UE::Slate::PrioritizedResize(
		AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, ClippingInfos, WrapButtonX
	);

	for (const UE::Slate::FClippingInfo& Info : ClippingInfos)
	{
		CHECK_FALSE(Info.bWasClipped);
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing with plenty of space does not modify input with default resize params", "[Slate]")
{
	constexpr float AllottedWidth = 1000.0f;
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(4.0f, 0);
	constexpr int32 WrapButtonIndex = -1;
	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f, .Width = 20.0f },
        UE::Slate::FClippingInfo{ .X = 20.0, .Width = 100.0f },
		UE::Slate::FClippingInfo{ .X = 120.0f, .Width = 30.0f },
        UE::Slate::FClippingInfo{ .X = 150.0f, .Width = 40.0f },
		UE::Slate::FClippingInfo{ .X = 190.0f, .Width = 200.0f },
        UE::Slate::FClippingInfo{ .X = 390.0f, .Width = 15.0f },
	};
	TOptional<float> WrapButtonX;

	TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
	UE::Slate::PrioritizedResize(
		AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
	);

	CHECK(OriginalClippingInfos.Num() == InOutClippingInfos.Num());

	for (int32 Index = 0; Index < OriginalClippingInfos.Num(); ++Index)
	{
		const UE::Slate::FClippingInfo Original = OriginalClippingInfos[Index];
		const UE::Slate::FClippingInfo Output = InOutClippingInfos[Index];

		CHECK(Original == Output);
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing with constrained size and two widgets clips the clippable one", "[Slate]")
{
	constexpr float AllottedWidth = 90.0f;
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(4.0f, 0);
	constexpr int32 WrapButtonIndex = -1;
	TArray<UE::Slate::FClippingInfo> InOutClippingInfos = {
		UE::Slate::FClippingInfo{ .ResizeParams = FMenuEntryResizeParams{ .AllowClipping = true },  .X = 0.0f,  .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = FMenuEntryResizeParams{ .AllowClipping = false }, .X = 50.0f, .Width = 50.0f },
	};
	TOptional<float> WrapButtonX;

	UE::Slate::PrioritizedResize(
		AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
	);

	CHECK(InOutClippingInfos[0].bWasClipped == true);
	CHECK(InOutClippingInfos[1].bWasClipped == false);
}

TEST_CASE("Runtime::Slate::PrioritizedResizing superfluous space after clipping is distributed to stretching widgets", "[Slate]")
{
	constexpr float WrapButtonWidth = 0.0;
	const FMargin WrapButtonPadding = FMargin(0);
	constexpr int32 WrapButtonIndex = -1;
	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 50.0f, .Width = 50.0f, .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .X = 100.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 150.0f, .Width = 50.0f, .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .X = 200.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 250.0f, .Width = 50.0f },
	};
	TOptional<float> WrapButtonX;

	// The tests following this rely on the total width of all widgets to be 300 pixels.
	{
		float TotalWidthOfWidgets = 0.0f;
		for (const UE::Slate::FClippingInfo& Info : OriginalClippingInfos)
		{
			TotalWidthOfWidgets += Info.Width;
		}

		CHECK(TotalWidthOfWidgets == 300.0f);
	}

	SECTION("When all widgets don't fit")
	{
		constexpr float AllottedWidth = 295.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		SECTION("Only the last one is clipped")
		{
			for (int Index = 0; Index < InOutClippingInfos.Num() - 1; ++Index)
			{
				INFO("Index=" << Index);
				CHECK(InOutClippingInfos[Index].bWasClipped == false);
			}

			CHECK(InOutClippingInfos.Last().bWasClipped);
		}

		SECTION("The non-stretching widgets keep their original width")
		{
			CHECK(InOutClippingInfos[0].Width == 50.0f);
			// Stretching widget
			CHECK(InOutClippingInfos[2].Width == 50.0f);
			// Stretching widget
			CHECK(InOutClippingInfos[4].Width == 50.0f);
			CHECK(InOutClippingInfos[5].Width == 50.0f);
		}

		SECTION("The stretching widgets are expanded")
		{
			// Non-stretching widget
			CHECK(InOutClippingInfos[1].Width > 51.0f);
			// Non-stretching widget
			CHECK(InOutClippingInfos[3].Width > 51.0f);
			// Non-stretching widget
			// Non-stretching widget
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing all superfluous space is eaten by stretching widgets", "[Slate]")
{
	constexpr float WrapButtonWidth = 0.0f;
	const FMargin WrapButtonPadding = FMargin(0.0f);
	constexpr int32 WrapButtonIndex = -1;
	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 50.0f, .Width = 50.0f, .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .X = 100.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams{ .ClippingPriority = -100.0f }, .X = 150.0f, .Width = 50.0f }, // Clips  first.
		UE::Slate::FClippingInfo{ .X = 200.0f, .Width = 50.0f },
	};
	TOptional<float> WrapButtonX;

	SECTION("When all widgets don't fit")
	{
		constexpr float AllottedWidth = 245.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		SECTION("The stretching widget eats all space")
		{
			CHECK(InOutClippingInfos[1].Width == AllottedWidth - 50.0f - 50.0f - 50.0f);
		}

		SECTION("The widget to the right of the stretching widget is moved to the right")
		{
			CHECK(InOutClippingInfos[2].X == AllottedWidth - 50.0f - 50.0f);
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing sorts zero-width widgets identical to input regardless of clipping priority", "[Slate]")
{
	constexpr float WrapButtonWidth = 0.0f;
	const FMargin WrapButtonPadding = FMargin(0.0f);
	constexpr int32 WrapButtonIndex = -1;

	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .ResizeParams{ .ClippingPriority = 2 }, .X = 0.0f, .Width = 0.0f,   .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .ResizeParams{ .ClippingPriority = 4 }, .X = 0.0f, .Width = 0.0f,   .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .ResizeParams{ .ClippingPriority = 3 }, .X = 0.0f, .Width = 100.0f, .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .ResizeParams{ .ClippingPriority = 1 }, .X = 0.0f, .Width = 0.0f,   .bIsStretchable = true },
	};
	TOptional<float> WrapButtonX;

	SECTION("When size is constrained")
	{
		constexpr float AllottedWidth = 50.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		SECTION("The widget sorting hasn't changed")
		{
			CHECK(InOutClippingInfos[0].ResizeParams.ClippingPriority.Get() == 2);
			CHECK(InOutClippingInfos[1].ResizeParams.ClippingPriority.Get() == 4);
			CHECK(InOutClippingInfos[2].ResizeParams.ClippingPriority.Get() == 3);
			CHECK(InOutClippingInfos[3].ResizeParams.ClippingPriority.Get() == 1);
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing positions a wrap button at index 1 snugly", "[Slate]")
{
	constexpr float WrapButtonWidth = 40;
	const FMargin WrapButtonPadding = FMargin(0);
	constexpr int32 WrapButtonIndex = 1;

	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f,   .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 50.0f,  .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 100.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 150.0f, .Width = 50.0f },
	};
	TOptional<float> WrapButtonX;

	SECTION("When size is constrained")
	{
		constexpr float AllottedWidth = 190.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		SECTION("The wrap button is positioned snugly with the non-clipped widgets")
		{
			CHECK(InOutClippingInfos[0].X == 0.0f);
			CHECK(InOutClippingInfos[0].Width == 50.0f);

			CHECK(WrapButtonX == 50);

			CHECK(InOutClippingInfos[1].X == 90.0f);
			CHECK(InOutClippingInfos[1].Width == 50.0f);

			CHECK(InOutClippingInfos[2].X == 140.0f);
			CHECK(InOutClippingInfos[2].Width == 50.0f);
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing adds wrap button when needed", "[Slate]")
{
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(0.0f);
	constexpr int32 WrapButtonIndex = 1;

	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .VisibleInOverflow = false }, .X = 50.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .VisibleInOverflow = false }, .X = 100.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .X = 150.0f, .Width = 50.0f },
	};
	TOptional<float> WrapButtonX;

	SECTION("When size is constrained")
	{
		constexpr float AllottedWidth = 190.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		SECTION("The non-right-most widgets are not clipped")
		{
			CHECK(InOutClippingInfos[0].bWasClipped == false);
			CHECK(InOutClippingInfos[1].bWasClipped == false);
			CHECK(InOutClippingInfos[2].bWasClipped == false);
		}

		SECTION("The right-most widget is clipped")
		{
			CHECK(InOutClippingInfos[3].bWasClipped == true);
		}

		SECTION("The wrap button is positioned")
		{
			CHECK(WrapButtonX == 50.0f);
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing with complex set of entries", "[Slate]")
{
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(0.0f);
	constexpr int32 WrapButtonIndex = 1;

	// clang-format off
	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 1000 }, .X = 0.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 500, .VisibleInOverflow = false }, .X = 50.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 500, .VisibleInOverflow = false }, .X = 100, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 800 }, .X = 150.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 400, .VisibleInOverflow = false }, .X = 200.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 400, .VisibleInOverflow = false }, .X = 250.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .AllowClipping = false }, .X = 300.0f, .Width = 50.0f, .bIsStretchable = true },
		UE::Slate::FClippingInfo{ .ResizeParams = { .ClippingPriority = 1000, .VisibleInOverflow = false }, .X = 350.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .AllowClipping = false }, .X = 400.0f, .Width = 50.0f },
		UE::Slate::FClippingInfo{ .ResizeParams = { .AllowClipping = false, .VisibleInOverflow = false  }, .X = 450.0f, .Width = 50.0f },
	};
	// clang-format on
	TOptional<float> WrapButtonX;

	GIVEN("Size is not constrained")
	{
		constexpr float AllottedWidth = 501.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("No widgets are clipped")
		{
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				CHECK(Info.bWasClipped == false);
			}
		}

		THEN("The wrap button is not placed")
		{
			CHECK(!WrapButtonX.IsSet());
		}
	}

	GIVEN("Size is constrained by one pixel")
	{
		constexpr float AllottedWidth = 499.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("The right-most of the lowest priority entries is clipped")
		{
			CHECK(InOutClippingInfos[5].bWasClipped == true);
		}

		THEN("Only one entry is clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 1);
		}

		THEN("The wrap button is not placed")
		{
			CHECK(!WrapButtonX.IsSet());
		}
	}

	GIVEN("Size is constrained by a little more than one button width")
	{
		constexpr float AllottedWidth = 445.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("Two entries are clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 2);
		}

		THEN("The wrap button is not placed")
		{
			CHECK(!WrapButtonX.IsSet());
		}
	}

	GIVEN("Size is constrained by a little more than two button widths")
	{
		constexpr float AllottedWidth = 395.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("Three entries are clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 3);
		}

		THEN("The wrap button is not placed")
		{
			CHECK(!WrapButtonX.IsSet());
		}
	}

	GIVEN("Size is constrained by a little more three button widths")
	{
		constexpr float AllottedWidth = 345.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("Four entries are clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 4);
		}

		THEN("The wrap button is not placed")
		{
			CHECK(!WrapButtonX.IsSet());
		}
	}
}

TEST_CASE("Runtime::Slate::PrioritizedResizing takes the wrapping button width into account", "[Slate]")
{
	constexpr float WrapButtonWidth = 40.0f;
	const FMargin WrapButtonPadding = FMargin(3.0f, 0.0f);
	constexpr int32 WrapButtonIndex = -2;

	const TArray<UE::Slate::FClippingInfo> OriginalClippingInfos = {
		UE::Slate::FClippingInfo{ .X = 0.0f,   .Width = 70.0f },
		UE::Slate::FClippingInfo{ .X = 70.0f,  .Width = 70.0f },
		UE::Slate::FClippingInfo{ .X = 140.0f, .Width = 70.0f },
		UE::Slate::FClippingInfo{ .X = 210.0f, .Width = 70.0f },
	};
	
	TOptional<float> WrapButtonX;

	GIVEN("Size is constrained by a few pixels")
	{
		constexpr float AllottedWidth = 274.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("The wrapping button is placed")
		{
			// Position + Left padding
			CHECK(WrapButtonX.Get(0.0f) == 140.0f + 3.0f);
		}

		THEN("The last widget is clipped")
		{
			CHECK(InOutClippingInfos[3].bWasClipped);
		}

		THEN("Only one widget is clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 1);
		}
	}
	
	WrapButtonX.Reset();

	GIVEN("Size is constrained enough so the wrap button will force another widget to clip")
	{
		constexpr float AllottedWidth = 230.0f;

		TArray<UE::Slate::FClippingInfo> InOutClippingInfos = OriginalClippingInfos;
		UE::Slate::PrioritizedResize(
			AllottedWidth, WrapButtonWidth, WrapButtonPadding, WrapButtonIndex, InOutClippingInfos, WrapButtonX
		);

		THEN("The wrapping button is placed")
		{
			// Position + Left padding
			CHECK(WrapButtonX.Get(0.0f) == 70.0f + 3.0f);
		}

		THEN("The second last widget is clipped")
		{
			CHECK(InOutClippingInfos[2].bWasClipped);
		}

		THEN("The last widget is clipped")
		{
			CHECK(InOutClippingInfos[3].bWasClipped);
		}

		THEN("Two widgets are clipped")
		{
			int32 NumClipped = 0;
			for (const UE::Slate::FClippingInfo& Info : InOutClippingInfos)
			{
				if (Info.bWasClipped)
				{
					++NumClipped;
				}
			}
			CHECK(NumClipped == 2);
		}
	}
}
