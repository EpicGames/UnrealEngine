// Copyright Epic Games, Inc. All Rights Reserved.

#if !EXPLICIT_TESTS_TARGET && WITH_LOW_LEVEL_TESTS

#include "TestCommon/CoreUtilities.h"
#include "TestCommon/CoreUObjectUtilities.h"

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	InitTaskGraph();
	InitThreadPool(true);
#if WITH_COREUOBJECT
	InitCoreUObject();
#endif
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
#if WITH_COREUOBJECT
	CleanupCoreUObject();
#endif
	CleanupThreadPool();
	CleanupTaskGraph();
}

#endif // !EXPLICIT_TESTS_TARGET && WITH_LOW_LEVEL_TESTS
